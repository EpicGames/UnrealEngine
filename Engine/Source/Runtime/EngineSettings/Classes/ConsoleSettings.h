// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "ConsoleSettings.generated.h"


/**
 * Structure for auto-complete commands and their descriptions.
 */
USTRUCT()
struct FAutoCompleteCommand
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(config, EditAnywhere, Category=Command)
	FString Command;

	UPROPERTY(config, EditAnywhere, Category=Command)
	FString Desc;

	FColor Color = FColor(180, 180, 180);

	bool operator<(const FAutoCompleteCommand& rhs) const
	{
		return Command < rhs.Command;
	}

	// for game console 
	const FString& GetLeft() const
	{
		return IsHistory() ? Desc : Command;
	}

	// for game console 
	const FString& GetRight() const
	{
		return IsHistory() ? Command : Desc;
	}

	// @return true:history, false: autocompletion
	bool IsHistory() const
	{
		return Desc == TEXT(">");
	}
	void SetHistory()
	{
		Desc = TEXT(">");
	}
};


/**
 * Implements the settings for the UConsole class.
 */
UCLASS(config=Input, defaultconfig, MinimalAPI)
class UConsoleSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** The maximum number of lines the console keeps in its output history. */
	UPROPERTY(globalconfig, EditAnywhere, Category=General)
	int32 MaxScrollbackSize;

	/** Manual list of auto-complete commands and info specified in BaseInput.ini */
	UPROPERTY(config, EditAnywhere, Category=AutoComplete)
	TArray<struct FAutoCompleteCommand> ManualAutoCompleteList;

	/** List of relative paths (e.g. Content/Maps) to search for map names for auto-complete usage. Specified in BaseInput.ini. */
	UPROPERTY(config, EditAnywhere, Category=AutoComplete)
	TArray<FString> AutoCompleteMapPaths;

	/** Amount of transparency of the console background. */
	UPROPERTY(config, EditAnywhere, Category=Colors, meta=(UIMin="0", UIMax="100", ClampMin="0", ClampMax="100"))
	float BackgroundOpacityPercentage;

	/** Whether we legacy bottom-to-top ordering or regular top-to-bottom ordering */
	UPROPERTY(config, EditAnywhere, Category = AutoComplete)
	bool bOrderTopToBottom;

	/** Display the first line of any available help text in the auto-complete window, if a description isn't available */
	UPROPERTY(config, EditAnywhere, Category = AutoComplete)
	bool bDisplayHelpInAutoComplete = true;

	/** The color used for text input. */
	UPROPERTY(config, EditAnywhere, Category=Colors)
	FColor InputColor;

	/** The color used for the previously typed commands history. */
	UPROPERTY(config, EditAnywhere, Category=Colors)
	FColor HistoryColor;

	/** The autocomplete color used for executable commands. */
	UPROPERTY(config, EditAnywhere, Category=Colors)
	FColor AutoCompleteCommandColor;

	/** The autocomplete color used for mutable CVars. */
	UPROPERTY(config, EditAnywhere, Category=Colors)
	FColor AutoCompleteCVarColor;

	/** The autocomplete color used for command descriptions and read-only CVars. */
	UPROPERTY(config, EditAnywhere, Category=Colors)
	FColor AutoCompleteFadedColor;

	/** Returns an array with the auto-complete command names that match all the words in the substring. */
	ENGINESETTINGS_API TArray<FString> GetFilteredManualAutoCompleteCommands(FStringView Substring) const;
};
