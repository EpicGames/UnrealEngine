// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "GameMapsSettings.generated.h"

/** Ways the screen can be split with two players. */
UENUM()
namespace ETwoPlayerSplitScreenType
{
	enum Type : int
	{
		Horizontal,
		Vertical
	};
}


/** Ways the screen can be split with three players. */
UENUM()
namespace EThreePlayerSplitScreenType
{
	enum Type : int
	{
		FavorTop,
		FavorBottom,
		Vertical,
		Horizontal
	};
}

UENUM()
enum class EFourPlayerSplitScreenType : uint8
{
	Grid,
	Vertical,
	Horizontal
};

/** Helper structure, used to associate GameModes with shortcut names. */
USTRUCT()
struct FGameModeName
{
	GENERATED_USTRUCT_BODY()

	/** Abbreviation/prefix that can be used as an alias for the class name */
	UPROPERTY(EditAnywhere, Category = DefaultModes, meta = (MetaClass = "/Script/Engine.GameModeBase"))
	FString Name;

	/** GameMode class to load */
	UPROPERTY(EditAnywhere, Category = DefaultModes, meta = (MetaClass = "/Script/Engine.GameModeBase"))
	FSoftClassPath GameMode;
};

UENUM()
enum class ESubLevelStripMode : uint8
{
	// The class of the sub level actor must be exactly this class
	ExactClass,

	// Any child class of this class will be stripped, this is more expensive than ExactClass
	IsChildOf
};


/** Used by new level dialog. */
USTRUCT()
struct FTemplateMapInfoOverride
{
	GENERATED_USTRUCT_BODY()

	/** The thumbnail to display in the new level dialog */
	UPROPERTY(config, EditAnywhere, Category = DefaultMaps, meta = (AllowedClasses = "/Script/Engine.Texture2D"))
	FSoftObjectPath Thumbnail;

	/** The path to the template map */
	UPROPERTY(config, EditAnywhere, Category = DefaultMaps, meta = (AllowedClasses = "/Script/Engine.World"))
	FSoftObjectPath Map;

	/** The display name of the map template in the level dialog. If this is empty the thumbnail name will be used */
	UPROPERTY(config, EditAnywhere, Category = DefaultMaps)
	FText DisplayName;

	FTemplateMapInfoOverride()
	{
	}
};

enum class EDefaultMapRequestType : uint8
{
	Default,
	Client,
	Server
};

UCLASS(config=Engine, defaultconfig, MinimalAPI)
class UGameMapsSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/**
	 * Get the default map specified in the settings.
	 * Makes a choice based on running as listen server/client vs dedicated server if RequestType is Default
	 *
	 * @param RequestType If default, it will return the map for your currently running configuration, otherwise you can request the client or server map specifically
	 * @return the default map specified in the settings
	 */
	static ENGINESETTINGS_API FString GetGameDefaultMap(EDefaultMapRequestType RequestType = EDefaultMapRequestType::Default);

	/**
	 * Get the global default game type specified in the configuration
	 * Makes a choice based on running as listen server/client vs dedicated server
	 * 
	 * @return the proper global default game type
	 */
	static ENGINESETTINGS_API FString GetGlobalDefaultGameMode( );

	/**
	 * Searches the GameModeClassAliases list for a named game mode, if not found will return passed in string
	 * 
	 * @return the proper game type class path to load
	 */
	static ENGINESETTINGS_API FString GetGameModeForName( const FString& GameModeName );

	/**
	 * Searches the GameModeMapPrefixes list for a named game mode, if not found will return passed in string
	 * 
	 * @return the proper game type class path to load, or empty if not found
	 */
	static ENGINESETTINGS_API FString GetGameModeForMapName( const FString& MapName );

	/**
	 * Set the default map to use (see GameDefaultMap below)
	 *
	 * @param NewMap name of valid map to use
	 */
	static ENGINESETTINGS_API void SetGameDefaultMap( const FString& NewMap );

	/**
	 * Set the default game type (see GlobalDefaultGameMode below)
	 *
	 * @param NewGameMode name of valid map to use
	 */
	static ENGINESETTINGS_API void SetGlobalDefaultGameMode( const FString& NewGameMode );

	ENGINESETTINGS_API virtual void PostInitProperties() override;
	ENGINESETTINGS_API virtual void PostReloadConfig( class FProperty* PropertyThatWasLoaded ) override;

public:

#if WITH_EDITORONLY_DATA
	/** If set, this map will be loaded when the Editor starts up. */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath EditorStartupMap;

	/** Map templates that should show up in the new level dialog. This will completely override the default maps chosen by the default editor */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, meta=(ConfigRestartRequired=true))
	TArray<FTemplateMapInfoOverride> EditorTemplateMapOverrides;
#endif

	/** The default options that will be appended to a map being loaded. */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, AdvancedDisplay)
	FString LocalMapOptions;

	/** The map loaded when transition from one map to another. */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, AdvancedDisplay, meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath TransitionMap;

	/** Whether the screen should be split or not when multiple local players are present */
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer)
	bool bUseSplitscreen;

	/** The viewport layout to use if the screen should be split and there are two local players */
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer, meta=(editcondition="bUseSplitScreen"))
	TEnumAsByte<ETwoPlayerSplitScreenType::Type> TwoPlayerSplitscreenLayout;

	/** The viewport layout to use if the screen should be split and there are three local players */
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer, meta=(editcondition="bUseSplitScreen"))
	TEnumAsByte<EThreePlayerSplitScreenType::Type> ThreePlayerSplitscreenLayout;

	/** The viewport layout to use if the screen should be split and there are three local players */
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer, meta=(editcondition="bUseSplitScreen"))
	EFourPlayerSplitScreenType FourPlayerSplitscreenLayout;

	/** Determines whether all player widget layers should be displayed in the main viewport when splitscreen is disabled. If unchecked, only Player 1 widget will be visible */
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer)
	bool bShowAllPlayerWidgetsWhenSplitscreenDisabled;

	/**
	* If enabled, this will make so that gamepads start being assigned to the second controller ID in local multiplayer games.
	* In PIE sessions with multiple windows, this has the same effect as enabling "Route 1st Gamepad to 2nd Client"
	*/
	UPROPERTY(config, EditAnywhere, Category=LocalMultiplayer, meta=(DisplayName="Skip Assigning Gamepad to Player 1"))
	bool bOffsetPlayerGamepadIds;

	/** The class to use when instantiating the transient GameInstance class */
	UPROPERTY(config, noclear, EditAnywhere, Category=GameInstance, meta=(MetaClass="/Script/Engine.GameInstance"))
	FSoftClassPath GameInstanceClass;

private:

	/** The map that will be loaded by default when no other map is loaded. */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath GameDefaultMap;

	/** The map that will be loaded by default when no other map is loaded (DEDICATED SERVER). */
	UPROPERTY(config, EditAnywhere, Category=DefaultMaps, AdvancedDisplay, meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath ServerDefaultMap;

	/** GameMode to use if not specified in any other way. (e.g. per-map DefaultGameMode or on the URL). */
	UPROPERTY(config, noclear, EditAnywhere, Category=DefaultModes, meta=(MetaClass="/Script/Engine.GameModeBase", DisplayName="Default GameMode"))
	FSoftClassPath GlobalDefaultGameMode;

	/**
	 * GameMode to use if not specified in any other way. (e.g. per-map DefaultGameMode or on the URL) (DEDICATED SERVERS)
	 * If not set, the GlobalDefaultGameMode value will be used.
	 */
	UPROPERTY(config, EditAnywhere, Category=DefaultModes, meta=(MetaClass="/Script/Engine.GameModeBase"), AdvancedDisplay)
	FSoftClassPath GlobalDefaultServerGameMode;

	/** Overrides the GameMode to use when loading a map that starts with a specific prefix */
	UPROPERTY(config, EditAnywhere, Category = DefaultModes, AdvancedDisplay)
	TArray<FGameModeName> GameModeMapPrefixes;

	/** List of GameModes to load when game= is specified in the URL (e.g. "DM" could be an alias for "MyProject.MyGameModeMP_DM") */
	UPROPERTY(config, EditAnywhere, Category = DefaultModes, AdvancedDisplay)
	TArray<FGameModeName> GameModeClassAliases;

public:

	/** Returns the game local maps settings */
	UFUNCTION(BlueprintPure, Category = Settings, meta=(DisplayName="Get Game Maps and Modes Settings"))
	static ENGINESETTINGS_API UGameMapsSettings* GetGameMapsSettings();

	/**
	 * Modify "Skip Assigning Gamepad to Player 1" GameMapsSettings option
	 * @param bSkipFirstPlayer		If set connected game pads will only be assigned to the second and subsequent players
	 * @note This value is saved to local config when changed.
	*/
	UFUNCTION(BlueprintCallable, Category = "Utilities")
	ENGINESETTINGS_API void SetSkipAssigningGamepadToPlayer1(bool bSkipFirstPlayer = true);

	UFUNCTION(BlueprintPure, Category = "Utilities")
	ENGINESETTINGS_API bool GetSkipAssigningGamepadToPlayer1() const;

};
