// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameNetworkManagerSettings.generated.h"


/**
 * Holds the settings for the AGameNetworkManager class.
 */
UCLASS(config=Game, MinimalAPI)
class UGameNetworkManagerSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Minimum bandwidth dynamically set per connection. */
	UPROPERTY(globalconfig, EditAnywhere, Category=Bandwidth)
	int32 MinDynamicBandwidth;

	/** Maximum bandwidth dynamically set per connection. */
	UPROPERTY(globalconfig, EditAnywhere, Category=Bandwidth)
	int32 MaxDynamicBandwidth;

	/** Total available bandwidth for listen server, split dynamically across net connections. */
	UPROPERTY(globalconfig, EditAnywhere, Category=Bandwidth)
	int32 TotalNetBandwidth;

	/** The point we determine the server is either delaying packets or has bad upstream. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	int32 BadPingThreshold;

	/** Used to determine if checking for standby cheats should occur. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	uint32 bIsStandbyCheckingEnabled:1;

	/**  The amount of time without packets before triggering the cheat code. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float StandbyRxCheatTime;

	/** The amount of time without packets before triggering the cheat code. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float StandbyTxCheatTime;

	/** The percentage of clients missing RX data before triggering the standby code. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float PercentMissingForRxStandby;

	/** The percentage of clients missing TX data before triggering the standby code. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float PercentMissingForTxStandby;

	/** The percentage of clients with bad ping before triggering the standby code. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float PercentForBadPing;

	/** The amount of time to wait before checking a connection for standby issues. */
	UPROPERTY(config, EditAnywhere, Category=StandbyCheats)
	float JoinInProgressStandbyWaitTime;
};
