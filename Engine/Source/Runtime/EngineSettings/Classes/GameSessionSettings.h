// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GameSessionSettings.generated.h"


UCLASS(config=Game, MinimalAPI)
class UGameSessionSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Maximum number of spectators allowed by this server. */
	UPROPERTY(globalconfig, EditAnywhere, Category=GameSessionSettings)
	int32 MaxSpectators;

	/** Maximum number of players allowed by this server. */
	UPROPERTY(globalconfig, EditAnywhere, Category=GameSessionSettings)
	int32 MaxPlayers;

    /** Is voice enabled always or via a push to talk key binding. */
	UPROPERTY(globalconfig, EditAnywhere, Category=GameSessionSettings)
	uint32 bRequiresPushToTalk:1;
};
