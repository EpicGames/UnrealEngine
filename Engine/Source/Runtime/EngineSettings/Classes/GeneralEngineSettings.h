// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "GeneralEngineSettings.generated.h"


UCLASS(config=Engine, MinimalAPI)
class UGeneralEngineSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

};
