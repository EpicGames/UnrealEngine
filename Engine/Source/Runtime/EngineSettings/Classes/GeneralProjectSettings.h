// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "GeneralProjectSettings.generated.h"


UCLASS(config=Game, defaultconfig, MinimalAPI)
class UGeneralProjectSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** The name of the company (author, provider) that created the project. */
	UPROPERTY(config, EditAnywhere, Category=Publisher)
	FString CompanyName;

	/** The Distinguished Name of the company (author, provider) that created the project, which is used by publishing tools on some platforms. */
	UPROPERTY(config, EditAnywhere, Category=Publisher)
	FString CompanyDistinguishedName;

	/** The project's copyright and/or trademark notices. */
	UPROPERTY(config, EditAnywhere, Category=Legal)
	FString CopyrightNotice;

	/** The project's description text. */
	UPROPERTY(config, EditAnywhere, Category=About)
	FString Description;

	/** The project's homepage URL. */
	UPROPERTY(config, EditAnywhere, Category=Publisher)
	FString Homepage;

	/** The project's licensing terms. */
	UPROPERTY(config, EditAnywhere, Category=Legal)
	FString LicensingTerms;

	/** The project's privacy policy. */
	UPROPERTY(config, EditAnywhere, Category=Legal)
	FString PrivacyPolicy;

	/** The project's unique identifier. */
	UPROPERTY(config, EditAnywhere, Category=About)
	FGuid ProjectID;

	/** The project's non-localized name. */
	UPROPERTY(config, EditAnywhere, Category=About)
	FString ProjectName;

	/** The project's version number. */
	UPROPERTY(config, EditAnywhere, Category=About)
	FString ProjectVersion;

	/** The project's support contact information. */
	UPROPERTY(config, EditAnywhere, Category=Publisher)
	FString SupportContact;

	/** The project's title as displayed on the window title bar (can include the tokens {GameName}, {PlatformArchitecture}, {BuildConfiguration} or {RHIName}, which will be replaced with the specified text) */
	UPROPERTY(config, EditAnywhere, Category=Displayed)
	FText ProjectDisplayedTitle;

	/** Additional data to be displayed on the window title bar in non-shipping configurations (can include the tokens {GameName}, {PlatformArchitecture}, {BuildConfiguration} or {RHIName}, which will be replaced with the specified text) */
	UPROPERTY(config, EditAnywhere, Category=Displayed)
	FText ProjectDebugTitleInfo;

	/** Should the game's window preserve its aspect ratio when resized by user. */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bShouldWindowPreserveAspectRatio;

	/** Should the game use a borderless Slate window instead of a window with system title bar and border */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bUseBorderlessWindow;

	/** Should the game attempt to start in VR, regardless of whether -vr was set on the commandline */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bStartInVR;
	
	/** Should the user be allowed to resize the window used by the game, when not using full screen */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bAllowWindowResize;

	/** Should a close button be shown for the game's window, when not using full screen */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bAllowClose;

	/** Should a maximize button be shown for the game's window, when not using full screen */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bAllowMaximize;

	/** Should a minimize button be shown for the game's window, when not using full screen */
	UPROPERTY(config, EditAnywhere, Category = Settings)
	bool bAllowMinimize;

	/*Determines the Eye offset of the virtual stereo device created when " -emulatestereo" command line arg is detected*/
	UPROPERTY(config, EditAnywhere, Category = Settings)
	float EyeOffsetForFakeStereoRenderingDevice;

	/*Determines the Field Of View of the virtual stereo device created when " -emulatestereo" command line arg is detected*/
	UPROPERTY(config, EditAnywhere, Category = Settings)
	float FOVForFakeStereoRenderingDevice;

	/* Determines how much of -emulatestereo FOV is above the horizontal plane (0-1) - usually helmets have a larger FOV down than up, so this value is less than 0.5. */
	UPROPERTY(config, EditAnywhere, Category = Settings, meta = (
		ClampMin = "0.01",
		ClampMax = "0.99"))
	float TopFOVRatioForFakeStereoRenderingDevice;

	/* Determines difference between left and eyes horizontal -emulatestereo FOVs. If 0, they are the same (their FOVs are symmetric), the closer to 1, the more each eye is looking towards its side. Reasonable values are in 0 - 0.4 range. */
	UPROPERTY(config, EditAnywhere, Category = Settings, meta=(
		ClampMin="-0.99",
		ClampMax="0.99"))
	float DifferenceBetweenEyesForFakeStereoRenderingDevice;
};
