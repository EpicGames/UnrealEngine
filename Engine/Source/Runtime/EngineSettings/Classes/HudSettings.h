// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "HudSettings.generated.h"


UCLASS(config=Game, MinimalAPI)
class UHudSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

	/** Whether the HUD is visible at all.	 */
	UPROPERTY(config, EditAnywhere, Category=HudSettings)
	uint32 bShowHUD:1;

	/** Collection of names specifying what debug info to display for ViewTarget actor.	 */
	UPROPERTY(globalconfig, EditAnywhere, Category=HudSettings)
	TArray<FName> DebugDisplay;
};
