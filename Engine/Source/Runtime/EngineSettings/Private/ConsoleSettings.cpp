// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConsoleSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConsoleSettings)

TArray<FString> UConsoleSettings::GetFilteredManualAutoCompleteCommands(FStringView Substring) const
{
	// We reserve some arbitrary size here, we could use more complex heuristics later on
	TArray<FString> Commands;
	Commands.Reserve(Substring.IsEmpty() ? ManualAutoCompleteList.Num() : 20);

	// We convert to FString because there's no ParseIntoArrayWS just yet that's compatible with String Views
	TArray<FString> Tokens;
	FString(Substring).ParseIntoArrayWS(Tokens);

	for (const FAutoCompleteCommand& Command : ManualAutoCompleteList)
	{
		bool bAllMatch = true;

		for (const FString& Token : Tokens)
		{
			if (!Command.Command.Contains(Token))
			{
				bAllMatch = false;
				break;
			}
		}

		if (bAllMatch)
		{
			Commands.Add(Command.Command);
		}
	}

	return Commands;
}
