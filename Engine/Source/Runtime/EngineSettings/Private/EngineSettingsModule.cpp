// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "UObject/UnrealType.h"
#include "ConsoleSettings.h"
#include "GameNetworkManagerSettings.h"
#include "GameMapsSettings.h"
#include "GameSessionSettings.h"
#include "GeneralEngineSettings.h"
#include "GeneralProjectSettings.h"
#include "HudSettings.h"
#include "Misc/ConfigCacheIni.h"


/**
 * Implements the EngineSettings module.
 */
class FEngineSettingsModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule( ) override { }
	virtual void ShutdownModule( ) override { }

	virtual bool SupportsDynamicReloading( ) override
	{
		return true;
	}
};


/* Class constructors
 *****************************************************************************/

UConsoleSettings::UConsoleSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


UGameMapsSettings::UGameMapsSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bUseSplitscreen(true)
	, TwoPlayerSplitscreenLayout(ETwoPlayerSplitScreenType::Horizontal)
	, ThreePlayerSplitscreenLayout(EThreePlayerSplitScreenType::FavorTop)
	, bShowAllPlayerWidgetsWhenSplitscreenDisabled(false)
	, bOffsetPlayerGamepadIds(false)
{ }


UGameNetworkManagerSettings::UGameNetworkManagerSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


UGameSessionSettings::UGameSessionSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


UGeneralEngineSettings::UGeneralEngineSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


UGeneralProjectSettings::UGeneralProjectSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
	, bShouldWindowPreserveAspectRatio(true)
	, bUseBorderlessWindow(false)
	, bStartInVR(false)
	, bAllowWindowResize(true)
	, bAllowClose(true)
	, bAllowMaximize(true)
	, bAllowMinimize(true)
	, EyeOffsetForFakeStereoRenderingDevice(0.0320000005f)
	, FOVForFakeStereoRenderingDevice(90)
	, TopFOVRatioForFakeStereoRenderingDevice(0.47f)
	, DifferenceBetweenEyesForFakeStereoRenderingDevice(0.2f)
{ }


UHudSettings::UHudSettings( const FObjectInitializer& ObjectInitializer )
	: Super(ObjectInitializer)
{ }


/* Static functions
 *****************************************************************************/

FString UGameMapsSettings::GetGameDefaultMap(EDefaultMapRequestType RequestType)
{
	switch (RequestType)
	{
		case EDefaultMapRequestType::Client:
			return GetDefault<UGameMapsSettings>()->GameDefaultMap.GetLongPackageName();

		case EDefaultMapRequestType::Server:
			return GetDefault<UGameMapsSettings>()->ServerDefaultMap.GetLongPackageName();

		case EDefaultMapRequestType::Default:
		default:
			return IsRunningDedicatedServer()
				? GetDefault<UGameMapsSettings>()->ServerDefaultMap.GetLongPackageName()
				: GetDefault<UGameMapsSettings>()->GameDefaultMap.GetLongPackageName();
	};
}


FString UGameMapsSettings::GetGlobalDefaultGameMode( )
{
	UGameMapsSettings* GameMapsSettings = Cast<UGameMapsSettings>(UGameMapsSettings::StaticClass()->GetDefaultObject());

	return IsRunningDedicatedServer() && GameMapsSettings->GlobalDefaultServerGameMode.IsValid()
		? GameMapsSettings->GlobalDefaultServerGameMode.ToString()
		: GameMapsSettings->GlobalDefaultGameMode.ToString();
}

FString UGameMapsSettings::GetGameModeForName(const FString& GameModeName)
{
	UGameMapsSettings* GameMapsSettings = Cast<UGameMapsSettings>(UGameMapsSettings::StaticClass()->GetDefaultObject());

	// Look to see if this should be remapped from a shortname to full class name
	for (const FGameModeName& Alias : GameMapsSettings->GameModeClassAliases)
	{
		if (GameModeName == Alias.Name)
		{
			// switch GameClassName to the full name
			return Alias.GameMode.ToString();
		}
	}

	// Check deprecated config
	const FConfigSection* GameModeSection = GConfig->GetSection(TEXT("/Script/Engine.GameMode"), false, GGameIni);
	
	if (GameModeSection)
	{
		TArray<FString> ConfigLines;
		GameModeSection->MultiFind(TEXT("GameModeClassAliases"), ConfigLines);

		if (ConfigLines.Num())
		{
			UE_LOGF(LogLoad, Warning, "GameMode::GameModeClassAliases are deprecated, move to GameMapsSettings");

			for (FString& ConfigString : ConfigLines)
			{
				FString ModeName, ModePath;
				if (FParse::Value(*ConfigString, TEXT("ShortName="), ModeName) && FParse::Value(*ConfigString, TEXT("GameClassName="), ModePath))
				{
					if (ModeName == GameModeName)
					{
						return ModePath;
					}
				}
			}

		}
	}

	return GameModeName;
}

FString UGameMapsSettings::GetGameModeForMapName(const FString& MapName)
{
	UGameMapsSettings* GameMapsSettings = Cast<UGameMapsSettings>(UGameMapsSettings::StaticClass()->GetDefaultObject());

	// See if we have a per-prefix default specified
	for (const FGameModeName& Prefix : GameMapsSettings->GameModeMapPrefixes)
	{
		if ((Prefix.Name.Len() > 0) && MapName.StartsWith(Prefix.Name))
		{
			return Prefix.GameMode.ToString();
		}
	}

	// Check deprecated config
	const FConfigSection* GameModeSection = GConfig->GetSection(TEXT("/Script/Engine.WorldSettings"), false, GGameIni);

	if (GameModeSection)
	{
		TArray<FString> ConfigLines;
		GameModeSection->MultiFind(TEXT("DefaultMapPrefixes"), ConfigLines);

		if (ConfigLines.Num())
		{
			UE_LOGF(LogLoad, Warning, "GameMode::DefaultMapPrefixes are deprecated, move to GameMapsSettings::GameModeMapPrefixes");

			for (FString& ConfigString : ConfigLines)
			{
				FString Prefix, ModePath;
				if (FParse::Value(*ConfigString, TEXT("Prefix="), Prefix) && FParse::Value(*ConfigString, TEXT("GameMode="), ModePath))
				{
					if (MapName.StartsWith(Prefix))
					{
						return ModePath;
					}
				}
			}

		}
	}

	return FString();
}

void UGameMapsSettings::SetGameDefaultMap( const FString& NewMap )
{
	UGameMapsSettings* GameMapsSettings = Cast<UGameMapsSettings>(UGameMapsSettings::StaticClass()->GetDefaultObject());

	if (IsRunningDedicatedServer())
	{
		GameMapsSettings->ServerDefaultMap = NewMap;
	}
	else
	{
		GameMapsSettings->GameDefaultMap = NewMap;
	}
}

void UGameMapsSettings::SetGlobalDefaultGameMode( const FString& NewGameMode )
{
	UGameMapsSettings* GameMapsSettings = Cast<UGameMapsSettings>(UGameMapsSettings::StaticClass()->GetDefaultObject());
	
	GameMapsSettings->GlobalDefaultGameMode = NewGameMode;
}

UGameMapsSettings* UGameMapsSettings::GetGameMapsSettings()
{
	return GetMutableDefault<UGameMapsSettings>();
}

void UGameMapsSettings::SetSkipAssigningGamepadToPlayer1(bool bSkipFirstPlayer)
{
	bOffsetPlayerGamepadIds = bSkipFirstPlayer;
}

bool UGameMapsSettings::GetSkipAssigningGamepadToPlayer1() const
{
	return bOffsetPlayerGamepadIds;
}

// Backwards compat for map strings
void FixMapAssetRef(FSoftObjectPath& MapAssetReference)
{
	const FString AssetRefStr = MapAssetReference.ToString();
	int32 DummyIndex;
	if (!AssetRefStr.IsEmpty() && !AssetRefStr.FindLastChar(TEXT('.'), DummyIndex))
	{
		FString MapName, MapPath;
		AssetRefStr.Split(TEXT("/"), &MapPath, &MapName, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		MapAssetReference.SetPath(FString::Printf(TEXT("%s/%s.%s"),*MapPath, *MapName, *MapName));
	}
};

void UGameMapsSettings::PostInitProperties()
{
	Super::PostInitProperties();

#if WITH_EDITORONLY_DATA
	FixMapAssetRef(EditorStartupMap);
#endif
	FixMapAssetRef(GameDefaultMap);
	FixMapAssetRef(ServerDefaultMap);
	FixMapAssetRef(TransitionMap);
}

void UGameMapsSettings::PostReloadConfig( FProperty* PropertyThatWasLoaded )
{
	if (PropertyThatWasLoaded)
	{
#if WITH_EDITORONLY_DATA
		if (PropertyThatWasLoaded->GetFName() == GET_MEMBER_NAME_CHECKED(UGameMapsSettings, EditorStartupMap))
		{
			FixMapAssetRef(EditorStartupMap);
		}
		else
#endif
		if (PropertyThatWasLoaded->GetFName() == GET_MEMBER_NAME_CHECKED(UGameMapsSettings, GameDefaultMap))
		{
			FixMapAssetRef(GameDefaultMap);
		}
		else if (PropertyThatWasLoaded->GetFName() == GET_MEMBER_NAME_CHECKED(UGameMapsSettings, ServerDefaultMap))
		{
			FixMapAssetRef(ServerDefaultMap);
		}
		else if (PropertyThatWasLoaded->GetFName() == GET_MEMBER_NAME_CHECKED(UGameMapsSettings, TransitionMap))
		{
			FixMapAssetRef(TransitionMap);
		}
	}
	else
	{
#if WITH_EDITORONLY_DATA
		FixMapAssetRef(EditorStartupMap);
#endif
		FixMapAssetRef(GameDefaultMap);
		FixMapAssetRef(ServerDefaultMap);
		FixMapAssetRef(TransitionMap);
	}
}


IMPLEMENT_MODULE(FEngineSettingsModule, EngineSettings);
