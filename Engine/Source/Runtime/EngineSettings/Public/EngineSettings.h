// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public Dependencies
 *****************************************************************************/

#include "CoreUObject.h"


/* Public Includes
 *****************************************************************************/

#include "ConsoleSettings.h"
#include "GameNetworkManagerSettings.h"
#include "GameMapsSettings.h"
#include "GameSessionSettings.h"
#include "GeneralEngineSettings.h"
#include "GeneralProjectSettings.h"
#include "HudSettings.h"
